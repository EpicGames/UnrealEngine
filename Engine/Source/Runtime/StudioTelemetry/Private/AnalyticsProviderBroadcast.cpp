// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnalyticsProviderBroadcast.h"
#include "Analytics.h"
#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IAnalyticsProviderModule.h"
#include "AnalyticsProviderConfigurationDelegate.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "HttpModule.h"
#include "HttpManager.h"

static FString ProviderSection;

FString GetAnalyticsProviderConfiguration(const FString& Name, bool)
{
	FString Result;
	GConfig->GetString(*ProviderSection, *Name, Result, GEngineIni);
	return Result;
}

TSharedPtr<FAnalyticsProviderBroadcast> FAnalyticsProviderBroadcast::CreateAnalyticsProvider()
{
	return MakeShared<FAnalyticsProviderBroadcast>();
}

TWeakPtr<IAnalyticsProvider> FAnalyticsProviderBroadcast::GetAnalyticsProvider(const FString& Name)
{
	TSharedPtr<IAnalyticsProvider>* ProviderPtr = Providers.Find(Name);
	return ProviderPtr != nullptr ? *ProviderPtr : TSharedPtr<IAnalyticsProvider>();
}

FAnalyticsProviderBroadcast::FAnalyticsProviderBroadcast()
{
	const FString TelemetryProviderSection("StudioTelemetry.Provider");

	TArray<FString> SectionNames;

	FString ProviderName;

	if (FParse::Value(FCommandLine::Get(), TEXT("ST_Provider="), ProviderName))
	{
		UE_LOGF(LogAnalytics, Display, "Selected Telemetry Provider %ls", *ProviderName);
	}
	
	if (GConfig->GetSectionNames(GEngineIni, SectionNames))
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.Find(TelemetryProviderSection) != INDEX_NONE)
			{
				ProviderSection = SectionName;

				FString UsageType;

				// Is this Provider enabled? A given application may choose to disable it 
				bool Enabled = true;
				Enabled = GConfig->GetBoolOrDefault(*ProviderSection, TEXT("Enabled"), true, GEngineIni);

				if (Enabled == false)
				{
					UE_LOGF(LogAnalytics, Display, "Provider %ls is disabled for this application", *ProviderSection);
					continue;
				}

				// Validate the usage type is for this build type
				if (GConfig->GetString(*ProviderSection, TEXT("UsageType"), UsageType, GEngineIni))
				{
					bool IsValidUseCase = false;

#if WITH_EDITOR
					// Must specify a Editor usage type for this type build
					if (UsageType.Find(TEXT("Editor")) != INDEX_NONE)
					{
						IsValidUseCase |= true;
					}
#else 
					// Must specify a Runtime usage type for all non Editor builds
					if (UsageType.Find(TEXT("Runtime")) != INDEX_NONE)
					{
						IsValidUseCase |= true;
					}
#endif

#if WITH_SERVER_CODE
					// Must specify a Server usage type for this type build
					if (UsageType.Find(TEXT("Server")) != INDEX_NONE)
					{
						IsValidUseCase |= true;
					}
#endif

#if WITH_CLIENT_CODE
					// Must specify a Client usage type for this type build
					if (UsageType.Find(TEXT("Client")) != INDEX_NONE)
					{
						IsValidUseCase |= true;
					}
#endif

					if (IsValidUseCase == false)
					{
						// This provider is not valid for this use case
						continue;
					}
				}
				else
				{
					// Must always specify a usage type
					UE_LOGF(LogAnalytics, Warning, "There must be a valid UsageType specified for analytics provider %ls", *ProviderSection);
					continue;
				}
				
				FString ProviderModuleName;

				if (GConfig->GetString(*ProviderSection, TEXT("ProviderModule"), ProviderModuleName, GEngineIni))
				{
					TSharedPtr<IAnalyticsProvider> AnalyticsProvider;

					FString Name = GetAnalyticsProviderConfiguration("Name", true);

					if ( Name.IsEmpty() )
					{ 
						UE_LOGF(LogAnalytics, Error, "There must be a valid Name specified for analytics provider %ls.", *ProviderSection);
						continue;
					}
					else if (Providers.Find(Name) )
					{
						UE_LOGF(LogAnalytics, Warning, "An analytics provider with name %ls already exists.", *Name);
						continue;
					}

					if (ProviderName.Len() && Name != ProviderName)
					{
						// Skip over this provider because we don't want to use it
						continue;
					}

					// Try to create the analytics provider
					AnalyticsProvider = FAnalytics::Get().CreateAnalyticsProvider(FName(ProviderModuleName), FAnalyticsProviderConfigurationDelegate::CreateStatic(&GetAnalyticsProviderConfiguration));
	
					if (AnalyticsProvider.IsValid())
					{
						UE_LOGF(LogAnalytics, Display, "Created an analytics provider %ls from module %ls configuration %ls [%ls]", *Name, *ProviderModuleName, *GEngineIni, *ProviderSection);
						Providers.Add(Name, AnalyticsProvider);
					}
					else
					{
						UE_LOGF(LogAnalytics, Warning, "Unable to create an analytics provider %ls from module %ls configuration %ls [%ls]", *Name, *ProviderModuleName, *GEngineIni, *ProviderSection);
					}
				}
				else
				{
					UE_LOGF(LogAnalytics, Warning, "There must be a valid ProviderModule specified for analytics provider %ls", *ProviderSection);
				}
			}
		}
	}

	if (ProviderName.Len() && Providers.IsEmpty())
	{
		// We were looking to use a named provider which did not exist so raise a warning
		UE_LOGF(LogAnalytics, Warning, "Unable to find a named analytics provider %ls", *ProviderName);
	}
}

bool FAnalyticsProviderBroadcast::SetSessionID(const FString& InSessionID)
{
	SessionID = InSessionID;

	bool bResult = true;

	for (TProviders::TConstIterator it(Providers); it; ++it)
	{
		bResult &= (*it).Value->SetSessionID(InSessionID);
	}

	return bResult;
}

FString FAnalyticsProviderBroadcast::GetSessionID() const
{
	return SessionID;
}

void FAnalyticsProviderBroadcast::SetUserID(const FString& InUserID)
{
	UserID = InUserID;

	for (TProviders::TConstIterator it(Providers); it; ++it)
	{
		(*it).Value->SetUserID(InUserID);
	}
}

FString FAnalyticsProviderBroadcast::GetUserID() const
{
	return UserID;
}

void FAnalyticsProviderBroadcast::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;

	for (TProviders::TConstIterator it(Providers); it; ++it)
	{
		(*it).Value->SetDefaultEventAttributes(CopyTemp(DefaultEventAttributes));
	}
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderBroadcast::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderBroadcast::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderBroadcast::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes.IsValidIndex(AttributeIndex)? DefaultEventAttributes[AttributeIndex] : FAnalyticsEventAttribute();
}

bool FAnalyticsProviderBroadcast::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
	bool bResult = true;

	for (TProviders::TConstIterator it(Providers);it;++it)
	{
		bResult &= (*it).Value->StartSession(Attributes);
	}
	return bResult;
}

void FAnalyticsProviderBroadcast::EndSession()
{
	for (TProviders::TConstIterator it(Providers);it;++it)
	{
		TSharedPtr<IAnalyticsProvider> Provider = (*it).Value;

		Provider->EndSession();
		Provider.Reset();
	}

	Providers.Reset();	
}

void FAnalyticsProviderBroadcast::FlushEvents()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAnalyticsProviderBroadcast::FlushEvents);

	for (TProviders::TConstIterator it(Providers); it; ++it)
	{
		(*it).Value->FlushEvents();
	}

	// It is quite likely that one of the analytics providers is sending data via the FHttpManager so we
	// need to flush that as well to make sure that the message is sent. 
	if (FHttpModule* HttpModule = FModuleManager::GetModulePtr<FHttpModule>("HTTP"))
	{
		FHttpManager& HttpManager = HttpModule->GetHttpManager();
		HttpManager.Flush(EHttpFlushReason::FullFlush);
	}
}

void FAnalyticsProviderBroadcast::SetRecordEventCallback(OnRecordEvent Callback)
{
	RecordEventCallback = Callback;
}

static void CheckForDuplicateAttributes(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT
	// Expose events that have duplicate attribute names. Not only does this provide ambiguous event data, in general this is not handled by all the analytics backends in any consistent manner.
	for (int32 indexA = 0; indexA < Attributes.Num(); ++indexA)
	{
		const FString& AttributeName = Attributes[indexA].GetName();

		for (int32 indexB = indexA + 1; indexB < Attributes.Num(); ++indexB)
		{
			const bool IsDuplicateAttribute = AttributeName == Attributes[indexB].GetName();

			if (IsDuplicateAttribute)
			{	
				UE_LOGF(LogAnalytics, Warning, "Duplicate Attributes %ls Found For Event %ls", *AttributeName, *EventName);
			}

			check(!IsDuplicateAttribute);
		}
	}
#endif
}

void FAnalyticsProviderBroadcast::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode)
{
	// Check for duplicate attributes against themselves
	CheckForDuplicateAttributes(EventName, Attributes);

	for (TProviders::TConstIterator it(Providers); it; ++it)
	{
		(*it).Value->RecordEvent(EventName, Attributes, Mode);
	}

	if (RecordEventCallback)
	{
		// Notify any callbacks
		RecordEventCallback(EventName, Attributes);
	}
}
