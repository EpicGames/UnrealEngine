// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsProviderConfigurationDelegate.h"
#include "Interfaces/IAnalyticsProvider.h"

/**
 * Implementation of the IAnalyticsProvider interface that forwards the API calls to an array of IAnalyticsProvider interfaces
 * 
 * IAnalyticsProviders are specified in the Engine.ini interface and are constructed and configured automatically.
 *
 * AnalyticsLog for example in Engine/Config/BaseEngine.ini
 * 
 * ; Studio Telemetry Settings
 * [StudioTelemetry.Log]
 * Name=LogAnalytics
 * ProviderModule=AnalyticsLog
 * UsageType=Editor
 * 
 * This instructs the Broadcast provider to create a Provider from the AnalyticsLog module via the IANalyticsProviderModule interface. 
 * See FAnalayticsLog and FAnalyticsProviderLog for more details.
 */
class FAnalyticsProviderBroadcast : public IAnalyticsProvider
{
public:

	using TProviders = TMap<FString, TSharedPtr<IAnalyticsProvider>>;

	typedef TFunction<void(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attrs)> OnRecordEvent;


	FAnalyticsProviderBroadcast();

	static TSharedPtr<FAnalyticsProviderBroadcast> CreateAnalyticsProvider();

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes = {}) override;
	virtual void EndSession() override;
	virtual void FlushEvents() override;
	
	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;

	virtual bool SetSessionID(const FString& InSessionID) override;
	virtual void SetUserID(const FString& InUserID) override;

	virtual FString GetSessionID() const override;
	virtual FString GetUserID() const override;
	
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override 
	{
		RecordEvent(EventName, Attributes, EAnalyticsRecordEventMode::Cached);
	}
	
	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes, EAnalyticsRecordEventMode Mode) override;	
	void SetRecordEventCallback(OnRecordEvent Callback);
	bool HasValidProviders() const { return Providers.Num() > 0; }
	TWeakPtr<IAnalyticsProvider> GetAnalyticsProvider(const FString& Name);

private:

	TProviders								Providers;
	FString									UserID;
	FString									SessionID;
	TArray<FAnalyticsEventAttribute>		DefaultEventAttributes;
	OnRecordEvent							RecordEventCallback;
};
