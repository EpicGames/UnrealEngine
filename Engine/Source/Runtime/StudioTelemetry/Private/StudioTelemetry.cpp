// Copyright Epic Games, Inc. All Rights Reserved.

#include "StudioTelemetry.h"
#include "StudioTelemetryLog.h"
#include "Analytics.h"
#include "AnalyticsProviderBroadcast.h"
#include "AnalyticsTracer.h"
#include "BuildSettings.h"

#include "HAL/PlatformTime.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"

#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/CoreDelegates.h"

#include "GenericPlatform/GenericPlatformMisc.h"

DEFINE_LOG_CATEGORY(LogStudioTelemetry);

IMPLEMENT_MODULE(FStudioTelemetry, StudioTelemetry)

FString GetEnvironmentVariable(const FString& CommandLineVar, const FString& EnvironmentVar)
{
	FString Result;
	if (false == FParse::Value(FCommandLine::Get(), *CommandLineVar, Result))
	{
		Result = FPlatformMisc::GetEnvironmentVariable(*EnvironmentVar);
	}
	return Result;
}

FStudioTelemetry& FStudioTelemetry::Get()
{
	static FStudioTelemetry StudioTelemetryInstance;
	return StudioTelemetryInstance;
}

void FStudioTelemetry::SetRecordEventCallback(OnRecordEventCallback Callback )
{
	RecordEventCallback = Callback;

	// If the provider already exists then set the callback
	if (AnalyticsProvider.IsValid())
	{	
		AnalyticsProvider->SetRecordEventCallback(RecordEventCallback);
	}
}


bool FStudioTelemetry::StartSession()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStudioTelemetry::StartSession);

	if (IsSessionRunning())
	{
		return true;
	}

	// Load the configuration
	LoadConfiguration();

	if (Config.bSendTelemetry == false)
	{
		// We did not wish to send any telemetry events
		return false;
	}

	FScopeLock ScopeLock(&CriticalSection);

	AnalyticsProvider = FAnalyticsProviderBroadcast::CreateAnalyticsProvider();

	if (AnalyticsProvider.IsValid())
	{
		TArray<FAnalyticsEventAttribute> DefaultAttributes;
		TArray<FAnalyticsEventAttribute> SessionContextAttributes;

		const FString UserID = FPlatformProcess::UserName(false);
		const FString ProjectName = FApp::GetProjectName();
		FString ComputerName = FPlatformProcess::ComputerName();

		FString ProjectIDString;
		GConfig->GetString(TEXT("/Script/EngineSettings.GeneralProjectSettings"), TEXT("ProjectID"), ProjectIDString, GGameIni);

		FGuid ProjectID;

		if (!ProjectIDString.IsEmpty())
		{
			TArray<FString> Elements;
			if (ProjectIDString.ParseIntoArray(Elements, TEXT("=")) == 5)
			{
				ProjectID = FGuid(FCString::Atoi(*(Elements[1])), FCString::Atoi(*(Elements[2])), FCString::Atoi(*(Elements[3])), FCString::Atoi(*(Elements[4])));
			}
			else
			{
				ProjectID = FGuid(ProjectIDString);
			}
		}

		FGuid SessionID = FApp::GetInstanceId();

		// Build the default attributes. These are sent with ALL events
		DefaultAttributes.Emplace(TEXT("Session_ID"), SessionID.ToString(EGuidFormats::DigitsWithHyphensInBraces));
		DefaultAttributes.Emplace(TEXT("Session_StartUTC"), FDateTime::UtcNow().ToUnixTimestampDecimal());

		FString SessionLabel;
		if (FParse::Value(FCommandLine::Get(), TEXT("SessionLabel="), SessionLabel))
		{
			DefaultAttributes.Emplace(TEXT("Session_Label"), SessionLabel);
		}

		// Build the session context attributes. These will always be sent with the SessionStart event but can also be appended to default attributes to send with all ALL events.
		SessionContextAttributes.Emplace(TEXT("ProjectName"), ProjectName);
		SessionContextAttributes.Emplace(TEXT("ProjectID"), ProjectID);

		SessionContextAttributes.Emplace(TEXT("Build_Configuration"), LexToString(FApp::GetBuildConfiguration()));
		SessionContextAttributes.Emplace(TEXT("Build_BranchName"), FApp::GetBranchName().ToLower());
		SessionContextAttributes.Emplace(TEXT("Build_Changelist"), BuildSettings::GetCurrentChangelist());

		SessionContextAttributes.Emplace(TEXT("Config_IsEditor"), GIsEditor);
		SessionContextAttributes.Emplace(TEXT("Config_IsBuildMachine"), GIsBuildMachine);
		SessionContextAttributes.Emplace(TEXT("Config_IsRunningCommandlet"), IsRunningCommandlet());
		SessionContextAttributes.Emplace(TEXT("Config_IsDebuggerPresent"), FPlatformMisc::IsDebuggerPresent());

		// Only send user data if requested
		if (Config.bSendUserData == true)
		{
			SessionContextAttributes.Emplace(TEXT("User_ID"), UserID);
			SessionContextAttributes.Emplace(TEXT("Application_Commandline"), FCommandLine::Get());
		}

		// ALways send the platform
		SessionContextAttributes.Emplace(TEXT("Hardware_Platform"), FString(FPlatformProperties::IniPlatformName()));

		// Only send detailed hardware data if requested
		if (Config.bSendHardwareData == true)
		{
			SessionContextAttributes.Emplace(TEXT("Hardware_CPU"), FPlatformMisc::GetCPUBrand());
			SessionContextAttributes.Emplace(TEXT("Hardware_CPU_Cores_Physical"), FPlatformMisc::NumberOfCores());
			SessionContextAttributes.Emplace(TEXT("Hardware_CPU_Cores_Logical"), FPlatformMisc::NumberOfCoresIncludingHyperthreads());
			SessionContextAttributes.Emplace(TEXT("Hardware_RAM"), static_cast<uint64>(FPlatformMemory::GetStats().TotalPhysical));
			SessionContextAttributes.Emplace(TEXT("Hardware_ComputerName"), ComputerName);
		}

		// Only send OS data if requested
		if (Config.bSendOSData == true)
		{
			FString OSVersionLabel;
			FString OSSubVersionLabel;

			FPlatformMisc::GetOSVersions(OSVersionLabel, OSSubVersionLabel);

			SessionContextAttributes.Emplace(TEXT("OS_Version"), FPlatformMisc::GetOSVersion());
			SessionContextAttributes.Emplace(TEXT("OS_VersionLabel"), OSVersionLabel);
			SessionContextAttributes.Emplace(TEXT("OS_VersionSubLabel"), OSSubVersionLabel);
			SessionContextAttributes.Emplace(TEXT("OS_ID"), FPlatformMisc::GetOperatingSystemId());
		}

		const FString HordeJobId = GetEnvironmentVariable(TEXT("HordeJobId="), TEXT("UE_HORDE_JOBID"));
		const FString HordeServerURL = GetEnvironmentVariable(TEXT("HordeServerUrl="), TEXT("UE_HORDE_URL") );

		// Send Horde environment if available
		if (!HordeServerURL.IsEmpty() && !HordeJobId.IsEmpty())
		{
			SessionContextAttributes.Emplace(TEXT("Horde_ServerURL"), HordeServerURL);
			SessionContextAttributes.Emplace(TEXT("Horde_JobID"), HordeJobId);
			SessionContextAttributes.Emplace(TEXT("Horde_StepID"), GetEnvironmentVariable(TEXT("HordeStepId="), TEXT("UE_HORDE_STEPID")));
			SessionContextAttributes.Emplace(TEXT("Horde_StepName"), GetEnvironmentVariable(TEXT("HordeStepName="), TEXT("UE_HORDE_STEPNAME")));
			SessionContextAttributes.Emplace(TEXT("Horde_TemplateName"), GetEnvironmentVariable(TEXT("HordeTemplateName="), TEXT("UE_HORDE_TEMPLATENAME")));	
		}

		// Send the Session Context attributes with ALL events by appending to the default attributes if required
		if (Config.bSendSessionContext == true)
		{
			DefaultAttributes.Append(SessionContextAttributes);
		}

		// Set up the analytics provider
		AnalyticsProvider->SetUserID(UserID);
		AnalyticsProvider->SetSessionID(SessionID.ToString(EGuidFormats::DigitsWithHyphensInBraces));
		AnalyticsProvider->SetDefaultEventAttributes(MoveTemp(DefaultAttributes));
		AnalyticsProvider->SetRecordEventCallback(RecordEventCallback);

		// Start the analytics session
		AnalyticsProvider->StartSession();

		// Create the IAnalyticsTracer interface
		AnalyticsTracer = FAnalytics::Get().CreateAnalyticsTracer();
		AnalyticsTracer->SetProvider(AnalyticsProvider);
		AnalyticsTracer->StartSession();

		// Bind the pre-exit callback
		FCoreDelegates::OnEnginePreExit.AddRaw(&FStudioTelemetry::Get(), &FStudioTelemetry::EndSession);

		SessionStartTime = FDateTime::UtcNow();
		TArray<FAnalyticsEventAttribute> Attributes;

		if (Config.bSendSessionContext == false)
		{
			// We always send the session context with the SessionStart event so if we have opted out of sending it with the DefaultAttributes to ALL events we need append it here 
			Attributes = SessionContextAttributes;
		}

		const int32 SessionContextSchemaVersion = 2;

		Attributes.Emplace(TEXT("SchemaVersion"), SessionContextSchemaVersion);

		AnalyticsProvider->RecordEvent(TEXT("StudioTelemetry.SessionStart"), Attributes);

		OnStartSession.Broadcast();

		UE_LOGF(LogStudioTelemetry, Log, "Started StudioTelemetry Session");

		return true;
	}

	return false;
}

void FStudioTelemetry::EndSession()
{
	if (IsSessionRunning())
	{
		FScopeLock ScopeLock(&CriticalSection);

		OnEndSession.Broadcast();

		// End session for the tracer and the provider
		if (AnalyticsTracer.IsValid())
		{
			AnalyticsTracer->EndSession();
			AnalyticsTracer.Reset();
		}

		if (AnalyticsProvider.IsValid())
		{
			const FTimespan SessionDuration = FDateTime::UtcNow() - SessionStartTime;

			TArray<FAnalyticsEventAttribute> Attributes; 
			Attributes.Emplace(TEXT("SchemaVersion"), 1);
			Attributes.Emplace(TEXT("Duration"), SessionDuration.GetTotalSeconds());
			AnalyticsProvider->RecordEvent(TEXT("StudioTelemetry.SessionEnd"), Attributes);

			AnalyticsProvider->FlushEvents();
			AnalyticsProvider->EndSession();
			AnalyticsProvider.Reset();
		}

		UE_LOGF(LogStudioTelemetry, Log, "Ended StudioTelemetry Session");
	}
}

bool FStudioTelemetry::IsSessionRunning() const
{
	return AnalyticsProvider.IsValid() && AnalyticsProvider->HasValidProviders();
}

void FStudioTelemetry::LoadConfiguration()
{
	const FString TelemetryConfigurationSection("StudioTelemetry.Config");

	// Look for the configuration settings in the Engine.ini files
	TArray<FString> SectionNames;

	if (GConfig->GetSectionNames(GEngineIni, SectionNames))
	{
		for (const FString& SectionName : SectionNames)
		{
			if (SectionName.Find(TelemetryConfigurationSection) != INDEX_NONE)
			{
				GConfig->GetBool(*SectionName, TEXT("SendTelemetry"), Config.bSendTelemetry, GEngineIni);
				GConfig->GetBool(*SectionName, TEXT("SendUserData"), Config.bSendUserData, GEngineIni);
				GConfig->GetBool(*SectionName, TEXT("SendHardwareData"), Config.bSendHardwareData, GEngineIni);
				GConfig->GetBool(*SectionName, TEXT("SendOSData"), Config.bSendOSData, GEngineIni);
				GConfig->GetBool(*SectionName, TEXT("SendSessionContext"), Config.bSendSessionContext, GEngineIni);
				
			}
		}
	}

	// Parse the commandline for any local configuration overrides
	FParse::Bool(FCommandLine::Get(), TEXT("ST_SendTelemetry="), Config.bSendTelemetry);
	FParse::Bool(FCommandLine::Get(), TEXT("ST_SendUserData="), Config.bSendUserData);
	FParse::Bool(FCommandLine::Get(), TEXT("ST_SendHardwareData="), Config.bSendHardwareData);
	FParse::Bool(FCommandLine::Get(), TEXT("ST_SendOSData="), Config.bSendOSData);
	FParse::Bool(FCommandLine::Get(), TEXT("ST_SendSessionContext="), Config.bSendSessionContext);

	
}

void FStudioTelemetry::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->RecordEvent(EventName, Attributes);
		OnRecordEvent.Broadcast(EventName, Attributes);
	}
}

void FStudioTelemetry::RecordEvent(const FName CategoryName, const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->RecordEvent(EventName, Attributes);
		OnRecordEvent.Broadcast(EventName, Attributes);
	}
}

void FStudioTelemetry::FlushEvents()
{
	FScopeLock ScopeLock(&CriticalSection);
	if (AnalyticsProvider.IsValid())
	{
		AnalyticsProvider->FlushEvents();
	}
}

void FStudioTelemetry::RecordEventToProvider(const FString& ProviderName, const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
	FScopeLock ScopeLock(&CriticalSection);
	TSharedPtr<IAnalyticsProvider> NamedProvider = GetProvider(ProviderName).Pin();

	if (NamedProvider.IsValid())
	{
		NamedProvider->RecordEvent(EventName, Attributes);
	}
}

TWeakPtr<IAnalyticsProvider> FStudioTelemetry::GetProvider()
{
	return AnalyticsProvider;
}

TWeakPtr<IAnalyticsProvider> FStudioTelemetry::GetProvider(const FString& Name)
{
	return AnalyticsProvider.IsValid()? AnalyticsProvider->GetAnalyticsProvider(Name) : TWeakPtr<IAnalyticsProvider>();
}

TWeakPtr<IAnalyticsTracer> FStudioTelemetry::GetTracer()
{
	return AnalyticsTracer;
}

TSharedPtr<IAnalyticsSpan> FStudioTelemetry::StartSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->StartSpan(Name, AdditionalAttributes) : TSharedPtr<IAnalyticsSpan>();
}

TSharedPtr<IAnalyticsSpan> FStudioTelemetry::StartSpan(const FName Name, TSharedPtr<IAnalyticsSpan> ParentSpan, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->StartSpan(Name, ParentSpan, AdditionalAttributes)  : TSharedPtr<IAnalyticsSpan>();
}

bool FStudioTelemetry::EndSpan(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->EndSpan(Span, AdditionalAttributes) : false;
}

bool FStudioTelemetry::EndSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes)
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->EndSpan(Name, AdditionalAttributes) : false;
}

TSharedPtr<IAnalyticsSpan> FStudioTelemetry::GetSpan(const FName Name)
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->GetSpan(Name) : TSharedPtr<IAnalyticsSpan>();
}

TSharedPtr<IAnalyticsSpan> FStudioTelemetry::GetSessionSpan() const
{
	return AnalyticsTracer.IsValid() ? AnalyticsTracer->GetSessionSpan() : TSharedPtr<IAnalyticsSpan>();
}


