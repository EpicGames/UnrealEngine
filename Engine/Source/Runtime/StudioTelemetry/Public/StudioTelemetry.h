// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IAnalyticsTracer.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

class FAnalyticsProviderBroadcast;
/**
 * Studio Telemetry API
 * 
 * Notes:
 * Interface for adding Studio level Telemetry to products. Studio Telemetry will never function in shipping builds.
 * Developers are encouraged to add post their own development telemetry events via this API.
 * Developers can implement their own IAnalyticsProviderModule where custom recording of Studio Telemetry events to their own Analytics Backend is desired.
 * Custom AnalyticsProviders can be added to the plugin via the .ini. See FAnalyticsProviderLog or FAnalyticsProviderET for example.
 * Telemetry events are recored to all registered IAnalyticsProviders supplied in the .ini file using the FAnalyticsProviderBroadcast provider, except where specifically recorded with the RecordEvent(ProviderName,.. ) API below
 */
class FStudioTelemetry : public IModuleInterface
{
public:

	typedef TFunction<void(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attrs)> OnRecordEventCallback;

	/** Check whether the module is available*/
	static bool IsAvailable() { return FStudioTelemetry::Get().IsSessionRunning(); }

	/** Access to the module singleton*/
	static STUDIOTELEMETRY_API FStudioTelemetry& Get();

	/** Access to the a specific named analytics provider within the system*/
	STUDIOTELEMETRY_API TWeakPtr<IAnalyticsProvider> GetProvider(const FString& ProviderName);

	/** Access to the broadcast analytics provider for the system*/
	STUDIOTELEMETRY_API TWeakPtr<IAnalyticsProvider> GetProvider();

	/** Access to the tracer for the system*/
	STUDIOTELEMETRY_API TWeakPtr<IAnalyticsTracer> GetTracer();

	/** Starts a new analytics session. Returns true if the session was already running or started successfully*/
	STUDIOTELEMETRY_API bool StartSession();

	/** Ends an existing analytics session*/
	STUDIOTELEMETRY_API void EndSession();

	/** Is Session Running */
	STUDIOTELEMETRY_API bool IsSessionRunning() const;
	
	/** Thread safe method to record an event to all registered analytics providers*/
	STUDIOTELEMETRY_API void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = {});

	/** Thread safe method to record an event to all registered analytics providers*/
	STUDIOTELEMETRY_API void RecordEvent(const FName CategoryName, const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = {});

	/** Thread safe method to record an event to the specifically named analytics provider */
	STUDIOTELEMETRY_API void RecordEventToProvider(const FString& ProviderName, const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes = {});

	/** Thread safe method to flush all events on all registered analytics providers*/
	STUDIOTELEMETRY_API void FlushEvents();

	/** Start a new span specifying the parent*/
	STUDIOTELEMETRY_API TSharedPtr<IAnalyticsSpan> StartSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** Start a new span specifying the parent*/
	STUDIOTELEMETRY_API TSharedPtr<IAnalyticsSpan> StartSpan(const FName Name, TSharedPtr<IAnalyticsSpan> ParentSpan, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** End an existing span*/
	STUDIOTELEMETRY_API bool EndSpan(TSharedPtr<IAnalyticsSpan> Span, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** End an existing span by name*/
	STUDIOTELEMETRY_API bool EndSpan(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {});

	/** Get an active span by name, non active spans will not be available*/
	STUDIOTELEMETRY_API TSharedPtr<IAnalyticsSpan> GetSpan(const FName Name);

	/** Get the root session span*/
	STUDIOTELEMETRY_API TSharedPtr<IAnalyticsSpan> GetSessionSpan() const;

	/** Callback for interception of telemetry events recording that can be used by Developers to send telemetry events to their own back end, though it is recommended that Developers implement their own IAnalyticsProvider via their own IAnalyticsProviderModule*/
	STUDIOTELEMETRY_API void SetRecordEventCallback(OnRecordEventCallback);

	/** Delegates for event callbacks **/
	DECLARE_MULTICAST_DELEGATE(FOnStartSession);
	FOnStartSession& GetOnStartSession() { return OnStartSession; }

	DECLARE_MULTICAST_DELEGATE(FOnEndSession);
	FOnEndSession& GetOnEndSession() { return OnEndSession; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRecordEvent, const FString&, const TArray<FAnalyticsEventAttribute>&);
	FOnRecordEvent& GetOnRecordEvent() { return OnRecordEvent; }

	/** Scoped Session helper class **/
	class FSessionScope
	{
	public:
		FSessionScope()
		{
			FStudioTelemetry::Get().StartSession();
		}

		~FSessionScope()
		{
			FStudioTelemetry::Get().EndSession();
		}
	};

	/** Scoped Span helper class **/
	class FSpanScope
	{
	public:
		FSpanScope(const FName Name, const TArray<FAnalyticsEventAttribute>& AdditionalAttributes = {} )
		{
			if (FStudioTelemetry::Get().IsAvailable())
			{
				Span = FStudioTelemetry::Get().StartSpan(Name, AdditionalAttributes);
			}
		}

		~FSpanScope()
		{
			if (FStudioTelemetry::Get().IsAvailable())
			{
				FStudioTelemetry::Get().EndSpan(Span);
			}
		}
	private:

		TSharedPtr<IAnalyticsSpan> Span;
	};

private:

	/** Configure the plugin*/
	void LoadConfiguration();

	struct FConfig
	{
		bool bSendTelemetry = true; // Only send telemetry data if we have been requested to
		bool bSendUserData = false;  // Never send user data unless specifically asked to
		bool bSendHardwareData = false; // Never send hardware data unless specifically asked to4
		bool bSendOSData = false; // Never send operating system data unless specifically asked to
		bool bSendSessionContext = true; // Send the session context data with every event
	};
	
	FCriticalSection						CriticalSection;
	TSharedPtr<FAnalyticsProviderBroadcast>	AnalyticsProvider;
	TSharedPtr<IAnalyticsTracer>			AnalyticsTracer;
	OnRecordEventCallback					RecordEventCallback;
	FGuid									SessionGUID;
	FConfig									Config;
	FDateTime								SessionStartTime;
	FOnStartSession							OnStartSession;
	FOnEndSession							OnEndSession;
	FOnRecordEvent							OnRecordEvent;
};

// Useful macros for scoped sessions and spans
#define STUDIO_TELEMETRY_SESSION_SCOPE FStudioTelemetry::FSessionScope UE_JOIN(FSessionScope, __LINE__);
#define STUDIO_TELEMETRY_SPAN_SCOPE(Name) FStudioTelemetry::FSpanScope UE_JOIN(FSpanScope, __LINE__)(TEXT(#Name));
#define STUDIO_TELEMETRY_START_SPAN(Name) if (FStudioTelemetry::Get().IsAvailable()) { FStudioTelemetry::Get().StartSpan(TEXT(#Name));}
#define STUDIO_TELEMETRY_END_SPAN(Name) if (FStudioTelemetry::Get().IsAvailable()) { FStudioTelemetry::Get().EndSpan(TEXT(#Name));}
