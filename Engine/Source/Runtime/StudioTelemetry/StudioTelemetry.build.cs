// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class StudioTelemetry : ModuleRules
{
	public StudioTelemetry(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"HTTP",
                "BuildSettings",
                "Analytics",
				"AnalyticsCSV",
				"AnalyticsET",
				"AnalyticsHorde",
				"AnalyticsLog",
				"TelemetryUtils"
			}
		);
	}
}
